// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePlayerProxy.h"
#include "Modules/ModuleManager.h"
#include "MoviePlayerProxyServer.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, MoviePlayerProxy);

IMoviePlayerProxyServer* FMoviePlayerProxy::Server = nullptr;

void FMoviePlayerProxy::BlockingStarted()
{
	if (Server != nullptr)
	{
		Server->BlockingStarted();
	}
}

void FMoviePlayerProxy::BlockingTick()
{
	if (Server != nullptr)
	{
		Server->BlockingTick();
	}
}

void FMoviePlayerProxy::BlockingFinished()
{
}

void FMoviePlayerProxy::BlockingForceFinished()
{
	if (Server != nullptr)
	{
		Server->BlockingFinished();
	}
}

void FMoviePlayerProxy::SetIsSlateThreadAllowed(bool bInIsSlateThreadAllowed)
{
	if (Server != nullptr)
	{
		Server->SetIsSlateThreadAllowed(bInIsSlateThreadAllowed);
	}
}

void FMoviePlayerProxy::RegisterServer(IMoviePlayerProxyServer* InServer)
{
	Server = InServer;
}

void FMoviePlayerProxy::UnregisterServer()
{
	Server = nullptr;
}
