// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IMoviePlayerProxyServer;

/**
 * This provides a mechanism to connect the engine (the client) to a movie player (the server).
 * 
 * Engine code can call BlockingStarted/Tick/Finished around blocking areas.
 * 
 * The movie player can call RegisterServer/UnregisterServer
 * so it can receive the calls from the engine.
 */ 
class FMoviePlayerProxy
{
public:
	/** Call this before doing a blocking operation on the game thread so that the movie player can activate. */
	static MOVIEPLAYERPROXY_API void BlockingStarted();
	/** Call this periodically during a blocking operation on the game thread. */
	static MOVIEPLAYERPROXY_API void BlockingTick();
	/** Call this once the blocking operation is done to shut down the movie player. */
	static MOVIEPLAYERPROXY_API void BlockingFinished();
	/** Call this to make sure the movie player is no longer running. */
	static MOVIEPLAYERPROXY_API void BlockingForceFinished();
	/** Call this to prevent the movie player from using the Slate thread. */
	static MOVIEPLAYERPROXY_API void SetIsSlateThreadAllowed(bool bInIsSlateThreadAllowed);
	/** Call this to hook up a server. */
	static MOVIEPLAYERPROXY_API void RegisterServer(IMoviePlayerProxyServer* InServer);
	/** Call this to unregister the current server. */
	static MOVIEPLAYERPROXY_API void UnregisterServer();
	
private:
	/** Our current worker that handles blocks. */
	static IMoviePlayerProxyServer* Server;
};

/**
 * This is a helper class for FMoviePlayerProxy.
 * 
 * You can just add an instance of this somewhere like
 * FMoviePlayerProxyBlock MoviePlayerBlock;
 * 
 * It will automatically call FMoviePlayerProxy::BlockingStarted
 * and then call FMoviePlayerProxy::BlockingFinished when it goes out of scope.
 * 
 * You can also manually call Finish if you want to trigger this earlier.
 */
class FMoviePlayerProxyBlock
{
public:
	FMoviePlayerProxyBlock()
		: bIsBlockDone(false)
	{
		FMoviePlayerProxy::BlockingStarted();
	}

	~FMoviePlayerProxyBlock()
	{
		Finish();
	}

	/** Call this if you want to end the block before this object goes out of scope. */
	void Finish()
	{
		if (bIsBlockDone == false)
		{
			bIsBlockDone = true;
			FMoviePlayerProxy::BlockingFinished();
		}
	}

private:
	/** If false, then a block is still active and finish needs to be called.. */
	bool bIsBlockDone;
};


