// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Implement this interface if you want to handle what happens when the engine interacts
 * with the MoviePlayer, etc when the game thread is blocked.
 */
class IMoviePlayerProxyServer
{
public:
	/** Called before doing a blocking operation on the game thread occurs so that the movie player can activate. */
	virtual void BlockingStarted() = 0;
	/** Called periodically during a blocking operation on the game thread. */
	virtual void BlockingTick() = 0;
	/** Called once the blocking operation is done to shut down the movie player.*/
	virtual void BlockingFinished() = 0;
	/** Call this to prevent the movie player from using the Slate thread. */
	virtual void SetIsSlateThreadAllowed(bool bInIsSlateThreadAllowed) = 0;
};

